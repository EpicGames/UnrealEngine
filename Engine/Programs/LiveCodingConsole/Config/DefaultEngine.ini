[LiveCodingConsole]
bDisableActionLimit=false
ActionLimit=100
